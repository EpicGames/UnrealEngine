// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationTestToolset : ModuleRules
	{
		public AutomationTestToolset(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"ToolsetRegistry",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AutomationController",
					"AutomationTest",
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"Json",
					"SessionServices",
					"UnrealEd",
				}
			);
		}
	}
}
