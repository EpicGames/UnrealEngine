// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTestToolset.h"

#include "AutomationGroupFilter.h"
#include "AutomationControllerSettings.h"
#include "AutomationState.h"
#include "AutomationTestToolsetSubsystem.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Editor.h"
#include "IAutomationControllerManager.h"
#include "IAutomationReport.h"
#include "Misc/FilterCollection.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationTestToolsetTools, Log, All);

// Returns the subsystem only if both it and its automation controller are valid.
// Tools that need a working controller use this; DiscoverTests bypasses it since
// it must work before the controller is fully initialized.
static UAutomationTestToolsetSubsystem* GetSubsystem()
{
	if (GEditor == nullptr)
	{
		return nullptr;
	}
	UAutomationTestToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAutomationTestToolsetSubsystem>();
	if (Subsystem == nullptr || !Subsystem->GetAutomationController().IsValid())
	{
		return nullptr;
	}
	return Subsystem;
}

static FString JsonObjectToString(const TSharedRef<FJsonObject>& JsonObject)
{
	FString OutputString;
	TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutputString);
	FJsonSerializer::Serialize(JsonObject, Writer);
	return OutputString;
}

// Developer iteration recipe (Python, run via run_python_file or similar):
//
//   import importlib, my_pkg.tests.test_foo
//   importlib.reload(my_pkg.tests.test_foo)
//   unreal.AutomationTestToolset.discover_tests(b_force_rediscover=True)
//
// The existing PythonTestRunner stays put - its FPythonAutomationTest::GetTests
// re-calls unittest_runner.discover_tests, which walks the reloaded module dict
// and yields any added or removed test methods.
UToolCallAsyncResultString* UAutomationTestToolset::DiscoverTests(bool bForceRediscover)
{
	UAutomationTestToolsetSubsystem* Subsystem = GEditor ? GEditor->GetEditorSubsystem<UAutomationTestToolsetSubsystem>() : nullptr;
	if (Subsystem == nullptr)
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("AutomationTestToolsetSubsystem is not available"));
		return ErrorResult;
	}

	IAutomationControllerManagerPtr Controller = Subsystem->GetAutomationController();
	if (!bForceRediscover && Controller.IsValid() && Controller->GetFilteredReports().Num() > 0)
	{
		UToolCallAsyncResultString* Result = NewObject<UToolCallAsyncResultString>();
		Result->SetValue(TEXT("{\"status\": \"ready\"}"));
		return Result;
	}

	// Force-rediscover path: re-trigger the worker handshake so the controller re-polls
	// FAutomationTestFramework. Each registered test's GetTests() runs again, picking up
	// any Python modules that were reloaded mid-session.
	UToolCallAsyncResultString* Result = NewObject<UToolCallAsyncResultString>();
	Subsystem->SetPendingDiscoveryResult(Result);
	Subsystem->RequestWorkerDiscovery();
	return Result;
}

FString UAutomationTestToolset::ListTests(const FString& NameFilter, const FString& TagFilter, int32 Limit)
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		return TEXT("{\"tests\": [], \"total\": 0, \"returned\": 0}");
	}

	IAutomationControllerManagerPtr Controller = Subsystem->GetAutomationController();
	TArray<TSharedPtr<IAutomationReport>>& TopLevelReports = Controller->GetFilteredReports();

	TArray<TSharedPtr<IAutomationReport>> LeafReports;
	UAutomationTestToolsetSubsystem::CollectLeafReports(TopLevelReports, LeafReports);

	TArray<TSharedPtr<FJsonValue>> TestArray;
	int32 MatchCount = 0;

	for (const TSharedPtr<IAutomationReport>& Report : LeafReports)
	{
		if (!Report.IsValid())
		{
			continue;
		}

		const FString& FullPath = Report->GetFullTestPath();

		if (!NameFilter.IsEmpty() && !FullPath.Contains(NameFilter))
		{
			continue;
		}
		if (!TagFilter.IsEmpty() && !Report->GetTags().Contains(TagFilter))
		{
			continue;
		}

		MatchCount++;

		if (Limit > 0 && TestArray.Num() >= Limit)
		{
			continue;
		}

		TestArray.Add(MakeShared<FJsonValueString>(FullPath));
	}

	TSharedRef<FJsonObject> Root = MakeShared<FJsonObject>();
	Root->SetArrayField(TEXT("tests"), TestArray);
	Root->SetNumberField(TEXT("total"), MatchCount);
	Root->SetNumberField(TEXT("returned"), TestArray.Num());

	return JsonObjectToString(Root);
}

UToolCallAsyncResultString* UAutomationTestToolset::RunTests(const TArray<FString>& TestNames)
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("AutomationTestToolsetSubsystem is not available"));
		return ErrorResult;
	}

	if (TestNames.IsEmpty())
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("No test names provided"));
		return ErrorResult;
	}

	IAutomationControllerManagerPtr Controller = Subsystem->GetAutomationController();

	Controller->StopTests();
	Controller->SetEnabledTests(TestNames);

	UToolCallAsyncResultString* Result = NewObject<UToolCallAsyncResultString>();
	Subsystem->SetPendingRunResult(Result, TestNames);

	Controller->RunTests(true);

	// Enable completion polling only after RunTests has set the controller to Running.
	Subsystem->EnableRunResultPolling();

	return Result;
}

UToolCallAsyncResultString* UAutomationTestToolset::RunTestsByFilter(const FString& FilterExpression)
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("AutomationTestToolsetSubsystem is not available"));
		return ErrorResult;
	}

	if (FilterExpression.IsEmpty())
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("Filter expression is empty"));
		return ErrorResult;
	}

	// Parse the filter expression. Syntax mirrors AutomationCommandline.cpp's
	// GenerateTestNamesFromCommandLine so the LLM and the -ExecCmds=Automation
	// RunTests path accept the same input.
	TArray<FString> ArgumentNames;
	FilterExpression.ParseIntoArray(ArgumentNames, TEXT("+"), true);

	UAutomationControllerSettings* Settings =
		UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();

	TArray<FAutomatedTestFilter> FiltersList;
	for (FString ArgumentName : ArgumentNames)
	{
		ArgumentName.TrimStartAndEndInline();
		if (ArgumentName.IsEmpty())
		{
			continue;
		}

		const FString FilterPrefix = TEXT("StartsWith:");
		const FString GroupPrefix = TEXT("Group:");

		if (ArgumentName.StartsWith(FilterPrefix))
		{
			FString FilterName = ArgumentName.RightChop(FilterPrefix.Len()).TrimStart();
			if (!FilterName.EndsWith(TEXT(".")))
			{
				FilterName += TEXT(".");
			}
			FiltersList.Add(FAutomatedTestFilter(FilterName, true, false));
		}
		else if (ArgumentName.StartsWith(GroupPrefix))
		{
			FString GroupName = ArgumentName.RightChop(GroupPrefix.Len()).TrimStart();
			bool bFoundGroup = false;
			for (const FAutomatedTestGroup& GroupEntry : Settings->Groups)
			{
				if (GroupEntry.Name == GroupName)
				{
					bFoundGroup = true;
					FiltersList.Append(GroupEntry.Filters);
					break;
				}
			}
			if (!bFoundGroup)
			{
				UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
				ErrorResult->SetError(FString::Printf(TEXT("No matching automation test group named '%s'"), *GroupName));
				return ErrorResult;
			}
		}
		else
		{
			bool bMatchFromStart = false;
			bool bMatchFromEnd = false;
			if (ArgumentName.StartsWith(TEXT("^")))
			{
				bMatchFromStart = true;
				ArgumentName.RightChopInline(1);
			}
			if (ArgumentName.EndsWith(TEXT("$")))
			{
				bMatchFromEnd = true;
				ArgumentName.LeftChopInline(1);
			}
			FiltersList.Add(FAutomatedTestFilter(ArgumentName, bMatchFromStart, bMatchFromEnd));
		}
	}

	if (FiltersList.IsEmpty())
	{
		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(TEXT("Filter expression yielded no filters"));
		return ErrorResult;
	}

	IAutomationControllerManagerPtr Controller = Subsystem->GetAutomationController();
	Controller->StopTests();

	// Two-pass filter so the runner actually picks the right set:
	//   1. Widen the filter to the full tree and disable everything. The
	//      post-discovery state has bEnabled=true on every leaf, and the
	//      runner's selector (FAutomationReport::GetEnabledTestReports) walks
	//      the full ChildReports tree -- not FilteredChildReports -- collecting
	//      any leaf with bEnabled=true. If we skip this step, every test runs.
	//   2. Narrow to the filter expression and enable only the matching subset
	//      in a single recursion via SetVisibleTestsEnabled. This avoids the
	//      O(LeafReports * RequestedNames) per-leaf membership check that
	//      FAutomationReport::SetEnabledTests does in the explicit-names path.
	TSharedPtr<AutomationFilterCollection> EmptyFilters = MakeShared<AutomationFilterCollection>();
	Controller->SetFilter(EmptyFilters);
	Controller->SetVisibleTestsEnabled(false);

	TSharedPtr<AutomationFilterCollection> FilterCollection = MakeShared<AutomationFilterCollection>();
	TSharedPtr<FAutomationGroupFilter> GroupFilter = MakeShared<FAutomationGroupFilter>(FiltersList);
	FilterCollection->Add(GroupFilter);
	Controller->SetFilter(FilterCollection);
	Controller->SetVisibleTestsEnabled(true);

	TArray<FString> EnabledNames;
	Controller->GetEnabledTestNames(EnabledNames);

	if (EnabledNames.IsEmpty())
	{
		// Widen the tree back before returning; SetFilter above narrowed it.
		Controller->SetFilter(EmptyFilters);
		Controller->SetVisibleTestsEnabled(true);

		UToolCallAsyncResultString* ErrorResult = NewObject<UToolCallAsyncResultString>();
		ErrorResult->SetError(FString::Printf(TEXT("No automation tests matched filter '%s'"), *FilterExpression));
		return ErrorResult;
	}

	UToolCallAsyncResultString* Result = NewObject<UToolCallAsyncResultString>();
	Subsystem->SetPendingRunResult(Result, EnabledNames);
	Subsystem->SetRestoreFilterOnComplete(true);

	Controller->RunTests(true);
	Subsystem->EnableRunResultPolling();

	return Result;
}

FString UAutomationTestToolset::GetTestResults()
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		return TEXT("{\"tests\": [], \"passed\": 0, \"failed\": 0, \"skipped\": 0, \"total\": 0, \"duration\": 0.0}");
	}

	return UAutomationTestToolsetSubsystem::FormatResultsJson(Subsystem->GetAutomationController(), Subsystem->GetRunningTestNames());
}

FString UAutomationTestToolset::GetTestStatus()
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		return TEXT("{\"state\": \"Disabled\", \"numEnabled\": 0, \"numComplete\": 0, \"numPassed\": 0, \"numFailed\": 0}");
	}

	IAutomationControllerManagerPtr Controller = Subsystem->GetAutomationController();

	FString StateString = TEXT("Disabled");
	switch (Controller->GetTestState())
	{
	case EAutomationControllerModuleState::Ready:    StateString = TEXT("Ready");    break;
	case EAutomationControllerModuleState::Running:  StateString = TEXT("Running");  break;
	case EAutomationControllerModuleState::Disabled: StateString = TEXT("Disabled"); break;
	}

	TArray<TSharedPtr<IAutomationReport>> EnabledReports = Controller->GetEnabledReports();
	TArray<TSharedPtr<IAutomationReport>> LeafReports;
	UAutomationTestToolsetSubsystem::CollectLeafReports(EnabledReports, LeafReports);

	int32 NumComplete = 0;
	int32 NumPassed = 0;
	int32 NumFailed = 0;
	for (const TSharedPtr<IAutomationReport>& Report : LeafReports)
	{
		if (!Report.IsValid())
		{
			continue;
		}
		EAutomationState State = Report->GetState(0, 0);
		if (State == EAutomationState::Success)
		{
			NumComplete++;
			NumPassed++;
		}
		else if (State == EAutomationState::Fail)
		{
			NumComplete++;
			NumFailed++;
		}
		else if (State == EAutomationState::Skipped)
		{
			NumComplete++;
		}
	}

	TSharedRef<FJsonObject> Root = MakeShared<FJsonObject>();
	Root->SetStringField(TEXT("state"), StateString);
	Root->SetNumberField(TEXT("numEnabled"), Controller->GetEnabledTestsNum());
	Root->SetNumberField(TEXT("numComplete"), NumComplete);
	Root->SetNumberField(TEXT("numPassed"), NumPassed);
	Root->SetNumberField(TEXT("numFailed"), NumFailed);

	return JsonObjectToString(Root);
}

bool UAutomationTestToolset::StopTests()
{
	UAutomationTestToolsetSubsystem* Subsystem = GetSubsystem();
	if (Subsystem == nullptr)
	{
		return false;
	}

	Subsystem->GetAutomationController()->StopTests();

	UToolCallAsyncResultString* PendingResult = Subsystem->GetPendingRunResult();
	if (PendingResult != nullptr && !PendingResult->bIsComplete)
	{
		PendingResult->SetError(TEXT("Tests stopped by user"));
	}

	return true;
}
