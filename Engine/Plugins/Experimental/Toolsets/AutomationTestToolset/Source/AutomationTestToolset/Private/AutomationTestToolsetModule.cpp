// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTestToolsetModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FAutomationTestToolsetModule, AutomationTestToolset)

void FAutomationTestToolsetModule::StartupModule()
{
}

void FAutomationTestToolsetModule::ShutdownModule()
{
}
