// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTestToolsetSubsystem.h"

#include "AutomationTestToolset.h"
#include "AutomationState.h"
#include "Containers/Ticker.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Editor.h"
#include "IAutomationControllerModule.h"
#include "ISessionInfo.h"
#include "ISessionManager.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Misc/FilterCollection.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Subsystems/SubsystemCollection.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

static bool bEnableAutomationTestToolset = true;

static void OnEnableAutomationTestToolsetChanged(IConsoleVariable* Variable)
{
	if (GEditor == nullptr)
	{
		return;
	}

	if (UAutomationTestToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAutomationTestToolsetSubsystem>())
	{
		Subsystem->SetToolsetEnabled(Variable->GetBool());
	}
}

static FAutoConsoleVariableRef CVarEnableAutomationTestToolset(
	TEXT("AutomationTestToolset.Enable"),
	bEnableAutomationTestToolset,
	TEXT("Enable or disable AutomationTestToolset registration. When disabled, its tools will not appear in the MCP tool list."),
	FConsoleVariableDelegate::CreateStatic(&OnEnableAutomationTestToolsetChanged),
	ECVF_Default);

void UAutomationTestToolsetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	IAutomationControllerModule& ControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>("AutomationController");
	AutomationController = ControllerModule.GetAutomationController();

	TestsAvailableHandle = AutomationController->OnTestsAvailable().AddUObject(this, &UAutomationTestToolsetSubsystem::HandleTestsAvailable);
	TestsRefreshedHandle = AutomationController->OnTestsRefreshed().AddUObject(this, &UAutomationTestToolsetSubsystem::HandleTestsRefreshed);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UAutomationTestToolsetSubsystem::OnTick));

	SetToolsetEnabled(bEnableAutomationTestToolset);
}

void UAutomationTestToolsetSubsystem::Deinitialize()
{
	FTSTicker::RemoveTicker(TickerHandle);

	if (AutomationController.IsValid())
	{
		AutomationController->OnTestsAvailable().Remove(TestsAvailableHandle);
		AutomationController->OnTestsRefreshed().Remove(TestsRefreshedHandle);
	}

	if (PendingDiscoveryResult.IsValid() && !PendingDiscoveryResult->bIsComplete)
	{
		PendingDiscoveryResult->SetError(TEXT("Subsystem shutting down"));
	}
	PendingDiscoveryResult.Reset();

	if (PendingRunResult.IsValid() && !PendingRunResult->bIsComplete)
	{
		PendingRunResult->SetError(TEXT("Subsystem shutting down"));
	}
	PendingRunResult.Reset();

	SetToolsetEnabled(false);
	Super::Deinitialize();
}

void UAutomationTestToolsetSubsystem::SetToolsetEnabled(bool bEnabled)
{
	if (bEnabled && !bToolsetRegistered)
	{
		UToolsetRegistry::RegisterToolsetClass(UAutomationTestToolset::StaticClass());
		bToolsetRegistered = true;
	}
	else if (!bEnabled && bToolsetRegistered)
	{
		UToolsetRegistry::UnregisterToolsetClass(UAutomationTestToolset::StaticClass());
		bToolsetRegistered = false;
	}
}

void UAutomationTestToolsetSubsystem::RequestWorkerDiscovery()
{
	if (!SessionManager.IsValid())
	{
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		SessionManager = SessionServicesModule.GetSessionManager();
	}
	DiscoveryElapsedSeconds = 0.0f;
	bDiscoveryRequested = true;
	bWorkersRequested = false;
}

void UAutomationTestToolsetSubsystem::SetPendingDiscoveryResult(UToolCallAsyncResultString* InResult)
{
	if (PendingDiscoveryResult.IsValid() && !PendingDiscoveryResult->bIsComplete)
	{
		PendingDiscoveryResult->SetError(TEXT("Superseded by new discovery request"));
	}
	PendingDiscoveryResult = TStrongObjectPtr<UToolCallAsyncResultString>(InResult);
}

void UAutomationTestToolsetSubsystem::SetPendingRunResult(UToolCallAsyncResultString* InResult, const TArray<FString>& InTestNames)
{
	if (PendingRunResult.IsValid() && !PendingRunResult->bIsComplete)
	{
		PendingRunResult->SetError(TEXT("Superseded by new test run"));
	}
	PendingRunResult = TStrongObjectPtr<UToolCallAsyncResultString>(InResult);
	RunningTestNames = TSet<FString>(InTestNames);
}

UToolCallAsyncResultString* UAutomationTestToolsetSubsystem::GetPendingRunResult() const
{
	return PendingRunResult.Get();
}

void UAutomationTestToolsetSubsystem::EnableRunResultPolling()
{
	bWaitingForResults = true;
}

bool UAutomationTestToolsetSubsystem::OnTick(float DeltaTime)
{
	if (!AutomationController.IsValid())
	{
		return true;
	}

	IAutomationControllerModule::Get().Tick();
	AutomationController->Tick();

	// Discovery: find local session, request workers, wait for test list.
	// The timeout covers the entire process until HandleTestsRefreshed resolves
	// the pending discovery result.
	if (bDiscoveryRequested)
	{
		DiscoveryElapsedSeconds += DeltaTime;

		if (DiscoveryElapsedSeconds > DiscoveryTimeoutSeconds)
		{
			bDiscoveryRequested = false;
			if (PendingDiscoveryResult.IsValid() && !PendingDiscoveryResult->bIsComplete)
			{
				PendingDiscoveryResult->SetError(TEXT("Discovery timed out"));
				PendingDiscoveryResult.Reset();
			}
		}
		else if (SessionManager.IsValid())
		{
			TSharedPtr<ISessionInfo> SelectedSession = SessionManager->GetSelectedSession();
			if (!SelectedSession.IsValid())
			{
				TArray<TSharedPtr<ISessionInfo>> Sessions;
				SessionManager->GetSessions(Sessions);
				for (const TSharedPtr<ISessionInfo>& Session : Sessions)
				{
					if (Session->GetSessionId() == FApp::GetSessionId())
					{
						SessionManager->SelectSession(Session);
						SelectedSession = Session;
						break;
					}
				}
			}

			if (SelectedSession.IsValid() && !bWorkersRequested)
			{
				AutomationController->RequestAvailableWorkers(SelectedSession->GetSessionId());
				bWorkersRequested = true;
			}
		}
	}

	// Poll for test run completion.
	if (bWaitingForResults && AutomationController->GetTestState() != EAutomationControllerModuleState::Running)
	{
		CompleteTestRun();
	}

	return true;
}

void UAutomationTestToolsetSubsystem::HandleTestsAvailable(EAutomationControllerModuleState::Type State)
{
	if (State == EAutomationControllerModuleState::Ready)
	{
		bControllerReady = true;
	}
	else if (State == EAutomationControllerModuleState::Disabled)
	{
		bControllerReady = false;
	}
}

void UAutomationTestToolsetSubsystem::HandleTestsRefreshed()
{
	bControllerReady = true;

	if (bDiscoveryRequested)
	{
		bDiscoveryRequested = false;

		TSharedPtr<AutomationFilterCollection> Filters = MakeShared<AutomationFilterCollection>();
		AutomationController->SetFilter(Filters);
		AutomationController->SetVisibleTestsEnabled(true);

		if (PendingDiscoveryResult.IsValid() && !PendingDiscoveryResult->bIsComplete)
		{
			PendingDiscoveryResult->SetValue(TEXT("{\"status\": \"ready\"}"));
			PendingDiscoveryResult.Reset();
		}
	}
}

void UAutomationTestToolsetSubsystem::CompleteTestRun()
{
	bWaitingForResults = false;

	if (PendingRunResult.IsValid() && !PendingRunResult->bIsComplete)
	{
		PendingRunResult->SetValue(FormatResultsJson(AutomationController, RunningTestNames));
	}
	PendingRunResult.Reset();

	// RunTestsByFilter narrows GetFilteredReports() via SetFilter. Widen it back
	// to the post-discovery state (empty filter + all visible) so subsequent
	// ListTests / DiscoverTests calls see the full test set again.
	if (bRestoreFilterOnComplete && AutomationController.IsValid())
	{
		TSharedPtr<AutomationFilterCollection> EmptyFilters = MakeShared<AutomationFilterCollection>();
		AutomationController->SetFilter(EmptyFilters);
		AutomationController->SetVisibleTestsEnabled(true);
	}
	bRestoreFilterOnComplete = false;
}

void UAutomationTestToolsetSubsystem::CollectLeafReports(const TArray<TSharedPtr<IAutomationReport>>& Reports, TArray<TSharedPtr<IAutomationReport>>& OutLeaves)
{
	for (const TSharedPtr<IAutomationReport>& Report : Reports)
	{
		if (!Report.IsValid())
		{
			continue;
		}
		if (!Report->IsParent())
		{
			OutLeaves.Add(Report);
		}
		else
		{
			CollectLeafReports(Report->GetFilteredChildren(), OutLeaves);
		}
	}
}

FString UAutomationTestToolsetSubsystem::FormatResultsJson(const IAutomationControllerManagerPtr& Controller, const TSet<FString>& TestNames)
{
	const TArray<TSharedPtr<IAutomationReport>>& AllReports = Controller->GetFilteredReports();

	int32 Passed = 0;
	int32 Failed = 0;
	int32 Skipped = 0;
	float TotalDuration = 0.0f;

	TArray<TSharedPtr<IAutomationReport>> LeafReports;
	CollectLeafReports(AllReports, LeafReports);

	TArray<TSharedPtr<FJsonValue>> TestArray;

	for (const TSharedPtr<IAutomationReport>& Report : LeafReports)
	{
		if (!Report.IsValid() || !TestNames.Contains(Report->GetFullTestPath()))
		{
			continue;
		}

		EAutomationState State = Report->GetState(0, 0);
		const FAutomationTestResults& Results = Report->GetResults(0, 0);

		switch (State)
		{
		case EAutomationState::Success: Passed++;  break;
		case EAutomationState::Fail:    Failed++;  break;
		case EAutomationState::Skipped: Skipped++; break;
		default: break;
		}

		TotalDuration += Results.Duration;

		TSharedRef<FJsonObject> TestObject = MakeShared<FJsonObject>();
		TestObject->SetStringField(TEXT("name"), Report->GetFullTestPath());
		TestObject->SetStringField(TEXT("state"), AutomationStateToString(State));
		TestObject->SetNumberField(TEXT("duration"), Results.Duration);

		TArray<TSharedPtr<FJsonValue>> Errors;
		TArray<TSharedPtr<FJsonValue>> Warnings;
		for (const FAutomationExecutionEntry& Entry : Results.GetEntries())
		{
			if (Entry.Event.Type == EAutomationEventType::Error)
			{
				Errors.Add(MakeShared<FJsonValueString>(Entry.Event.Message));
			}
			else if (Entry.Event.Type == EAutomationEventType::Warning)
			{
				Warnings.Add(MakeShared<FJsonValueString>(Entry.Event.Message));
			}
		}
		TestObject->SetArrayField(TEXT("errors"), Errors);
		TestObject->SetArrayField(TEXT("warnings"), Warnings);

		TestArray.Add(MakeShared<FJsonValueObject>(TestObject));
	}

	TSharedRef<FJsonObject> Root = MakeShared<FJsonObject>();
	Root->SetArrayField(TEXT("tests"), TestArray);
	Root->SetNumberField(TEXT("passed"), Passed);
	Root->SetNumberField(TEXT("failed"), Failed);
	Root->SetNumberField(TEXT("skipped"), Skipped);
	Root->SetNumberField(TEXT("total"), TestArray.Num());
	Root->SetNumberField(TEXT("duration"), TotalDuration);

	FString OutputString;
	TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutputString);
	FJsonSerializer::Serialize(Root, Writer);
	return OutputString;
}
