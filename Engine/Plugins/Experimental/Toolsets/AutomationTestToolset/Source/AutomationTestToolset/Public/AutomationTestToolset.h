// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "AutomationTestToolset.generated.h"

class UToolCallAsyncResultString;

/**
 * Automation test discovery and execution toolset.
 *
 * Wraps the IAutomationControllerManager API (the same backend the Session
 * Frontend uses) to let MCP clients list available tests, run them, and
 * retrieve results.
 *
 * Typical workflow:
 *   1. DiscoverTests() once per session to initialize workers and load the test list.
 *   2. ListTests() to find tests by name or tag.
 *   3. RunTests() with the desired test names.
 *   4. GetTestStatus() / GetTestResults() to monitor and retrieve results.
 *   5. StopTests() to abort if needed.
 */
UCLASS(BlueprintType, MinimalAPI)
class UAutomationTestToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Initialize automation worker discovery and load the test list.
	 * Must be called once before ListTests or RunTests. Takes several seconds
	 * as it discovers the local automation worker and enumerates all registered
	 * tests. Returns an async result that completes with a JSON status object
	 * when tests are available, or an error if discovery fails.
	 * @param bForceRediscover  When true, bypass the cached report tree and re-poll
	 *   workers. Used after reloading Python test modules mid-session. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API UToolCallAsyncResultString* DiscoverTests(bool bForceRediscover = false);

	/** List available automation tests. Requires DiscoverTests() to have completed.
	 * Returns a JSON object: {"tests": ["path1", ...], "total": N, "returned": N}.
	 * @param NameFilter  Optional substring filter applied to the test's full path.
	 * @param TagFilter   Optional substring filter applied to the test's tags.
	 * @param Limit       Maximum number of tests to return (0 = unlimited, default 200). */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API FString ListTests(const FString& NameFilter = TEXT(""), const FString& TagFilter = TEXT(""), int32 Limit = 200);

	/** Run a set of automation tests by name. Requires DiscoverTests() to have completed.
	 * Starts executing the specified tests and returns an async result that
	 * completes with a JSON summary when all tests finish.
	 * @param TestNames  Array of full test paths as returned by ListTests. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API UToolCallAsyncResultString* RunTests(const TArray<FString>& TestNames);

	/** Run automation tests selected by a filter expression. Requires DiscoverTests()
	 * to have completed. Much faster than RunTests when targeting a large batch
	 * because the engine narrows the report tree in a single pass instead of
	 * running a per-leaf membership check against the requested name list.
	 *
	 * Filter syntax (multiple expressions joined by '+'):
	 *   "StartsWith:System.Engine"  prefix match against the full test path
	 *   "^Foo"                       prefix anchor (equivalent to StartsWith:)
	 *   "Bar$"                       suffix anchor
	 *   "Substring"                  bare token matches anywhere in the path
	 *   "Group:Smoke"                expand a named group from
	 *                                AutomationControllerSettings ini Groups
	 *
	 * Returns an async result that completes with the same JSON summary as RunTests.
	 * @param FilterExpression  Filter expression as described above. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API UToolCallAsyncResultString* RunTestsByFilter(const FString& FilterExpression);

	/** Get detailed results for the current or most recent test run.
	 * Requires DiscoverTests() to have completed.
	 * Returns a JSON object with per-test state, duration, errors, and warnings. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API FString GetTestResults();

	/** Get a lightweight status snapshot of the automation controller.
	 * Requires DiscoverTests() to have completed.
	 * Returns a JSON object with the controller state, enabled test count,
	 * and completion/pass/fail counts. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API FString GetTestStatus();

	/** Stop all currently running tests. Requires DiscoverTests() to have completed.
	 * If a RunTests async result is pending, it will be completed with an error.
	 * @return True if the stop request was issued successfully. */
	UFUNCTION(meta = (AICallable), Category = "AutomationTestToolset")
	static AUTOMATIONTESTTOOLSET_API bool StopTests();
};
