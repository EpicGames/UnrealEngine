// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "IAutomationControllerManager.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"
#include "UObject/StrongObjectPtr.h"

#include "AutomationTestToolsetSubsystem.generated.h"

class FSubsystemCollectionBase;
class ISessionManager;

/**
 * Manages the automation controller lifecycle for the AutomationTestToolset.
 *
 * Handles session discovery, worker communication, and test execution polling.
 * The toolset's static tool functions delegate to this subsystem for state management.
 */
UCLASS(MinimalAPI)
class UAutomationTestToolsetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	virtual void Deinitialize() override;

	/** Register or unregister the toolset with UToolsetRegistry at runtime. */
	void SetToolsetEnabled(bool bEnabled);

	/** Get the automation controller. */
	IAutomationControllerManagerPtr GetAutomationController() const { return AutomationController; }

	/** Whether workers have been found and the test list is populated. */
	bool IsControllerReady() const { return bControllerReady; }

	/** Load SessionServices and begin polling for the local session.
	 * Once found, requests available automation workers. */
	void RequestWorkerDiscovery();

	/** Store a pending async result that will resolve when test discovery completes. */
	void SetPendingDiscoveryResult(UToolCallAsyncResultString* InResult);

	/** Store a pending async result for the current test run.
	 * Resolves with results JSON when tests finish. */
	void SetPendingRunResult(UToolCallAsyncResultString* InResult, const TArray<FString>& InTestNames);

	/** Get the pending run result, if any. */
	UToolCallAsyncResultString* GetPendingRunResult() const;

	/** Start polling for test completion. Call after RunTests() has started execution. */
	void EnableRunResultPolling();

	/** Mark that the controller filter should be reset to empty and visible tests
	 *  re-enabled when the current run completes. RunTestsByFilter narrows the
	 *  report tree via SetFilter; this widens it back so subsequent ListTests
	 *  and DiscoverTests calls see the full test set. */
	void SetRestoreFilterOnComplete(bool bRestore) { bRestoreFilterOnComplete = bRestore; }

	/** Get the test names from the current or most recent run. */
	const TSet<FString>& GetRunningTestNames() const { return RunningTestNames; }

	/** Build a JSON results string for the given test names from the controller's report tree. */
	static FString FormatResultsJson(const IAutomationControllerManagerPtr& Controller, const TSet<FString>& TestNames);

	/** Recursively collect leaf (runnable) reports from a report tree. */
	static void CollectLeafReports(const TArray<TSharedPtr<IAutomationReport>>& Reports, TArray<TSharedPtr<IAutomationReport>>& OutLeaves);

private:

	static constexpr float DiscoveryTimeoutSeconds = 60.0f;

	bool OnTick(float DeltaTime);

	void HandleTestsAvailable(EAutomationControllerModuleState::Type State);

	void HandleTestsRefreshed();

	void CompleteTestRun();

	TSharedPtr<ISessionManager> SessionManager;

	IAutomationControllerManagerPtr AutomationController;

	TStrongObjectPtr<UToolCallAsyncResultString> PendingDiscoveryResult;

	TStrongObjectPtr<UToolCallAsyncResultString> PendingRunResult;

	TSet<FString> RunningTestNames;

	FTSTicker::FDelegateHandle TickerHandle;

	FDelegateHandle TestsAvailableHandle;

	FDelegateHandle TestsRefreshedHandle;

	float DiscoveryElapsedSeconds = 0.0f;

	bool bToolsetRegistered = false;

	bool bControllerReady = false;

	bool bDiscoveryRequested = false;

	bool bWorkersRequested = false;

	bool bWaitingForResults = false;

	bool bRestoreFilterOnComplete = false;
};
