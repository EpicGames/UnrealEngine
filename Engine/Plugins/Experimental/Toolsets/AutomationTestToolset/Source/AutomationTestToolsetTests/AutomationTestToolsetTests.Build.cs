// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationTestToolsetTests : ModuleRules
	{
		public AutomationTestToolsetTests(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AutomationController",
				"AutomationTestToolset",
				"Core",
				"CoreUObject",
				"CQTest",
				"Engine",
				"UnrealEd",
			});
		}
	}
}
