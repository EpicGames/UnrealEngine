// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

#include "AutomationTestToolset.h"
#include "AutomationTestToolsetSubsystem.h"
#include "Editor.h"
#include "ToolsetRegistry/ToolCallAsyncResultString.h"

TEST_CLASS(AutomationTestToolsetTest, "AI.Toolsets.AutomationTestToolset")
{
	// ── DiscoverTests ───────────────────────────────────────────────

	TEST_METHOD(DiscoverTests_ReturnsAsyncResult)
	{
		UToolCallAsyncResultString* Result = UAutomationTestToolset::DiscoverTests();
		ASSERT_THAT(IsNotNull(Result));
	}

	TEST_METHOD(DiscoverTests_SecondCallReturnsReadyImmediately)
	{
		// First call triggers discovery (or returns ready if already done).
		UToolCallAsyncResultString* First = UAutomationTestToolset::DiscoverTests();
		ASSERT_THAT(IsNotNull(First));

		// If the first call already completed synchronously, second should be instant.
		if (First->bIsComplete && First->Error.IsEmpty())
		{
			UToolCallAsyncResultString* Second = UAutomationTestToolset::DiscoverTests();
			ASSERT_THAT(IsNotNull(Second));
			ASSERT_THAT(IsTrue(Second->bIsComplete));
			ASSERT_THAT(IsTrue(Second->Error.IsEmpty()));
		}
	}

	// ── ListTests ───────────────────────────────────────────────────

	TEST_METHOD(ListTests_ReturnsJsonWithTestsArray)
	{
		const FString Result = UAutomationTestToolset::ListTests();
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"tests\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"total\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"returned\""))));
	}

	TEST_METHOD(ListTests_NameFilterReducesResults)
	{
		const FString AllTests = UAutomationTestToolset::ListTests(TEXT(""), TEXT(""), 0);
		const FString Filtered = UAutomationTestToolset::ListTests(TEXT("AutomationTestToolset"), TEXT(""), 0);

		// Filtered should have fewer or equal matches.
		ASSERT_THAT(IsTrue(Filtered.Len() <= AllTests.Len()));
	}

	TEST_METHOD(ListTests_LimitCapsResults)
	{
		const FString Result = UAutomationTestToolset::ListTests(TEXT(""), TEXT(""), 2);
		// The "returned" count should be at most 2.
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"returned\": 2")) || Result.Contains(TEXT("\"returned\": 1")) || Result.Contains(TEXT("\"returned\": 0"))));
	}

	// ── GetTestStatus ───────────────────────────────────────────────

	TEST_METHOD(GetTestStatus_ReturnsJsonWithState)
	{
		const FString Result = UAutomationTestToolset::GetTestStatus();
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"state\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"numEnabled\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"numComplete\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"numPassed\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"numFailed\""))));
	}

	// ── GetTestResults ──────────────────────────────────────────────

	TEST_METHOD(GetTestResults_ReturnsJsonWithTestsArray)
	{
		const FString Result = UAutomationTestToolset::GetTestResults();
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"tests\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"passed\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"failed\""))));
		ASSERT_THAT(IsTrue(Result.Contains(TEXT("\"total\""))));
	}

	// ── RunTests ────────────────────────────────────────────────────

	TEST_METHOD(RunTests_EmptyArrayReturnsError)
	{
		UToolCallAsyncResultString* Result = UAutomationTestToolset::RunTests({});
		ASSERT_THAT(IsNotNull(Result));
		ASSERT_THAT(IsTrue(Result->bIsComplete));
		ASSERT_THAT(IsTrue(!Result->Error.IsEmpty()));
	}

	// ── Subsystem ───────────────────────────────────────────────────

	TEST_METHOD(Subsystem_IsAvailable)
	{
		UAutomationTestToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAutomationTestToolsetSubsystem>();
		ASSERT_THAT(IsNotNull(Subsystem));
		ASSERT_THAT(IsTrue(Subsystem->GetAutomationController().IsValid()));
	}

};
