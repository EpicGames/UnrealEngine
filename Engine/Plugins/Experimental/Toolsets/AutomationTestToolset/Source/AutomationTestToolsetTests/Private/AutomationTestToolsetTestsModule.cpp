// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FAutomationTestToolsetTestsModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FAutomationTestToolsetTestsModule, AutomationTestToolsetTests)
